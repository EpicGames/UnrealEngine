{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "1.0",
	"FriendlyName": "MicrosoftSpatialSound",
	"Description": "Audio spatialization plugin using Microsoft's SASAPI service.",
	"Category": "Audio",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"CanContainContent": false,
	"IsBetaVersion": false,
	"Installed": false,
	"SupportedTargetPlatforms": [
		"Win64"
	],
	"Modules": [
		{
			"Name": "MicrosoftSpatialSound",
			"Type": "Runtime",
			"LoadingPhase": "PreDefault",
			"PlatformAllowList": [
				"Win64"
			]
		}
	]
}