// Copyright Epic Games, Inc. All Rights Reserved.
using UnrealBuildTool;
using System;
public class MicrosoftSpatialSound : ModuleRules
{
	protected virtual bool bSupportedPlatform { get => Target.Platform == UnrealTargetPlatform.Win64; }
	protected virtual string Platform { get => "Windows"; }

	protected virtual string MakeBinariesPath(ReadOnlyTargetRules Target, string EngineDir, string LibraryName)
	{
		string BinariesDir = System.IO.Path.Combine(EngineDir, "Binaries", "ThirdParty", LibraryName);

		string Config = (Target.Configuration == UnrealTargetConfiguration.Debug && Target.bDebugBuildsActuallyUseDebugCRT) ? "Debug" : "Release";
		string Arch = Target.Architecture.WindowsLibDir;
		string SubPath = System.IO.Path.Combine(Platform, Config, Arch);

		string BinariesPath = System.IO.Path.Combine(BinariesDir, SubPath);

		return BinariesPath;
	}

	public MicrosoftSpatialSound(ReadOnlyTargetRules Target) : base(Target)
	{
		PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;
		
		PublicIncludePaths.AddRange(
			new string[] {
			}
			);

		string EngineDir = System.IO.Path.GetFullPath(Target.RelativeEnginePath);

		string LibraryName = "SpatialAudioClientInterop";
		string BinariesPath = MakeBinariesPath(Target, EngineDir, LibraryName);
		string lib = String.Format("{0}.lib", LibraryName);
		string dll = String.Format("{0}.dll", LibraryName);

		if (bSupportedPlatform)
		{
			PublicAdditionalLibraries.Add(System.IO.Path.Combine(BinariesPath, lib));
			PublicDelayLoadDLLs.Add(dll);
			RuntimeDependencies.Add(System.IO.Path.Combine(BinariesPath, dll));
		}
		
		PrivateIncludePaths.AddRange(
			new string[] {
				System.IO.Path.Combine(EngineDir, "Source/ThirdParty/SpatialAudioClientInterop"),
			}
			);
			
		
		PublicDependencyModuleNames.AddRange(
			new string[]
			{
				"Core",
                "AudioExtensions"
            }
			);
			
		
		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"CoreUObject",
				"Engine",
			}
			);
        PrivateIncludePathModuleNames.Add("TargetPlatform");
    }
}