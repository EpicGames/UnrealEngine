// Copyright Epic Games, Inc. All Rights Reserved.

#include "MicrosoftSpatialSoundPlugin.h"
#include "AudioAnalytics.h"
#include "Features/IModularFeatures.h"
#include "HAL/PlatformMisc.h"
#include "Misc/Paths.h"

DEFINE_LOG_CATEGORY_STATIC(LogMicrosoftSpatialSound, Verbose, All);


#define MIN_WIN_10_VERSION_FOR_WMR_SPATSOUND 1809
#define WINDOWS_MIXED_REALITY_DEBUG_DLL_SPATSOUND 0

static const float UnrealUnitsToMeters = 0.01f;

static void LogMicrosoftSpatialAudioError(HRESULT Result, int32 LineNumber)
{
	FString ErrorString;

#if PLATFORM_WINDOWS
	switch (Result)
	{
		case REGDB_E_CLASSNOTREG:						
			ErrorString = TEXT("REGDB_E_CLASSNOTREG");
			break;

		case CLASS_E_NOAGGREGATION:
			ErrorString = TEXT("CLASS_E_NOAGGREGATION");
			break;

		case E_NOINTERFACE:
			ErrorString = TEXT("E_NOINTERFACE");
			break;

		case E_POINTER: 
			ErrorString = TEXT("E_POINTER");
			break;

		case E_INVALIDARG: 
			ErrorString = TEXT("E_INVALIDARG");
			break;

		case E_OUTOFMEMORY: 
			ErrorString = TEXT("E_OUTOFMEMORY");
			break;

		case AUDCLNT_E_UNSUPPORTED_FORMAT:
			ErrorString = TEXT("AUDCLNT_E_UNSUPPORTED_FORMAT");
			break;

		case SPTLAUDCLNT_E_DESTROYED:
			ErrorString = TEXT("SPTLAUDCLNT_E_DESTROYED");
			break;

		case SPTLAUDCLNT_E_OUT_OF_ORDER:
			ErrorString = TEXT("SPTLAUDCLNT_E_OUT_OF_ORDER");
			break;

		case SPTLAUDCLNT_E_RESOURCES_INVALIDATED:
			ErrorString = TEXT("SPTLAUDCLNT_E_RESOURCES_INVALIDATED");
			break;

		case SPTLAUDCLNT_E_NO_MORE_OBJECTS:
			ErrorString = TEXT("SPTLAUDCLNT_E_NO_MORE_OBJECTS");
			break;

		case SPTLAUDCLNT_E_PROPERTY_NOT_SUPPORTED:
			ErrorString = TEXT("SPTLAUDCLNT_E_PROPERTY_NOT_SUPPORTED");
			break;

		case SPTLAUDCLNT_E_ERRORS_IN_OBJECT_CALLS:
			ErrorString = TEXT("SPTLAUDCLNT_E_ERRORS_IN_OBJECT_CALLS");
			break;

		case SPTLAUDCLNT_E_METADATA_FORMAT_NOT_SUPPORTED:
			ErrorString = TEXT("SPTLAUDCLNT_E_METADATA_FORMAT_NOT_SUPPORTED");
			break;

		case SPTLAUDCLNT_E_STREAM_NOT_AVAILABLE:
			ErrorString = TEXT("SPTLAUDCLNT_E_STREAM_NOT_AVAILABLE");
			break;

		case SPTLAUDCLNT_E_INVALID_LICENSE:
			ErrorString = TEXT("SPTLAUDCLNT_E_INVALID_LICENSE");
			break;

		case SPTLAUDCLNT_E_STREAM_NOT_STOPPED:
			ErrorString = TEXT("SPTLAUDCLNT_E_STREAM_NOT_STOPPED");
			break;

		case SPTLAUDCLNT_E_STATIC_OBJECT_NOT_AVAILABLE:
			ErrorString = TEXT("SPTLAUDCLNT_E_STATIC_OBJECT_NOT_AVAILABLE");
			break;

		case SPTLAUDCLNT_E_OBJECT_ALREADY_ACTIVE:
			ErrorString = TEXT("SPTLAUDCLNT_E_OBJECT_ALREADY_ACTIVE");
			break;

		case SPTLAUDCLNT_E_INTERNAL:
			ErrorString = TEXT("SPTLAUDCLNT_E_INTERNAL");
			break;

		case AUDCLNT_E_BUFFER_ERROR:
			ErrorString = TEXT("AUDCLNT_E_BUFFER_ERROR");
			break;

		case AUDCLNT_E_BUFFER_TOO_LARGE:
			ErrorString = TEXT("AUDCLNT_E_BUFFER_TOO_LARGE");
			break;

		case AUDCLNT_E_BUFFER_SIZE_ERROR:
			ErrorString = TEXT("AUDCLNT_E_BUFFER_SIZE_ERROR");
			break;

		case AUDCLNT_E_OUT_OF_ORDER:
			ErrorString = TEXT("AUDCLNT_E_OUT_OF_ORDER");
			break;

		case AUDCLNT_E_DEVICE_INVALIDATED:
			ErrorString = TEXT("AUDCLNT_E_DEVICE_INVALIDATED");
			break;

		case AUDCLNT_E_BUFFER_OPERATION_PENDING:
			ErrorString = TEXT("AUDCLNT_E_BUFFER_OPERATION_PENDING");
			break;

		case AUDCLNT_E_SERVICE_NOT_RUNNING:
			ErrorString = TEXT("AUDCLNT_E_SERVICE_NOT_RUNNING");
			break;

		default: 
			ErrorString= FString::Printf(TEXT("UKNOWN (HRESULT=%d)"), (int32)Result);
			break;
	}
#else
	ErrorString = FString::Printf(TEXT("UKNOWN: '%d'"), (int32)Result);
#endif
	ErrorString = FString::Printf(TEXT("%s, line number %d"), *ErrorString, LineNumber);
	UE_LOGF(LogMicrosoftSpatialSound, Error, "%ls", *ErrorString);
}

// Macro to check result code for XAudio2 failure, get the string version, log, and goto a cleanup
#define MS_SPATIAL_AUDIO_RETURN_ON_FAIL(Result)					\
	if (FAILED(Result))											\
	{															\
		LogMicrosoftSpatialAudioError(Result, __LINE__);		\
		return;													\
	}


#define MS_SPATIAL_AUDIO_LOG_ON_FAIL(Result)					\
	if (FAILED(Result))											\
	{															\
		LogMicrosoftSpatialAudioError(Result, __LINE__);		\
	}

// Function which maps unreal coordinates to MS Spatial sound coordinates
static FORCEINLINE FVector UnrealToMicrosoftSpatialSoundCoordinates(const FVector& Input, float InDistance)
{
	return { UnrealUnitsToMeters * Input.Y * InDistance, UnrealUnitsToMeters * Input.X * InDistance, -UnrealUnitsToMeters * Input.Z * InDistance };
}

FMicrosoftSpatialSound::FMicrosoftSpatialSound()
	: MinFramesRequiredPerObjectUpdate(0)
	, MaxDynamicObjects(0)
	, SpatialAudioRenderThread(nullptr)
	, SAC(nullptr)
	, bIsInitialized(false)
{
}

//HACK: flag is static bool to avoid changing public header for 4.25.1.  It would be better as a member.
static bool bWarnedMicrosoftSpatialSoundDynamicObjectCountIsZero = false;

FMicrosoftSpatialSound::~FMicrosoftSpatialSound()
{
	Shutdown();
}


void FMicrosoftSpatialSound::FreeResources()
{
	if (SAC)
	{
		SAC->Release();
		SAC = nullptr;
	}
}

void FMicrosoftSpatialSound::Initialize(const FAudioPluginInitializationParams InitParams)
{
	InitializationParams = InitParams;

	// Spatial sound processes 1% of the frames per callback
	MinFramesRequiredPerObjectUpdate = InitParams.SampleRate / 100;

	SAC = SpatialAudioClient::CreateSpatialAudioClient();
	if (SAC)
	{
		bool bSuccess = SAC->Start(InitParams.NumSources, InitParams.SampleRate);

		if (bSuccess)
		{
			// Prepare our own record keeping for the number of spatial sources we expect to render
			Objects.Reset();
			Objects.AddDefaulted(InitializationParams.NumSources);

			// Flag that we're rendering
			bIsRendering = true;
			bIsInitialized = true;

			bWarnedMicrosoftSpatialSoundDynamicObjectCountIsZero = false;

			SpatialAudioRenderThread = FRunnableThread::Create(this, TEXT("MicrosoftSpatialAudioThread"), 0, TPri_TimeCritical, FPlatformAffinity::GetAudioRenderThreadMask());

			Audio::Analytics::RecordEvent_Usage(TEXT("MicrosoftSpatialSound.Initialized"));
		}
		else
		{
			FreeResources();
		}
	}
}

void FMicrosoftSpatialSound::Shutdown()
{
	if (bIsInitialized)
	{
		// Flag that we're no longer rendering
		bIsRendering = false;

		check(SpatialAudioRenderThread != nullptr);

		SpatialAudioRenderThread->Kill(true);
		delete SpatialAudioRenderThread;
		SpatialAudioRenderThread = nullptr;
	}

	FreeResources();
	bIsInitialized = false;
}

void FMicrosoftSpatialSound::OnInitSource(const uint32 SourceId, const FName& AudioComponentUserId, USpatializationPluginSourceSettingsBase* InSettings)
{
	if (!bIsInitialized)
	{
		return;
	}

	FScopeLock ScopeLock(&Objects[SourceId].ObjectCritSect);

	FSpatialSoundSourceObjectData& ObjectData = Objects[SourceId];
	check(!ObjectData.bActive);
}

void FMicrosoftSpatialSound::OnReleaseSource(const uint32 SourceId)
{
	if (!bIsInitialized)
	{
		return;
	}

	FScopeLock ScopeLock(&Objects[SourceId].ObjectCritSect);

	Objects[SourceId].bActive = false;
	Objects[SourceId].bBuffering = false;
}

void FMicrosoftSpatialSound::ProcessAudio(const FAudioPluginSourceInputData& InputData, FAudioPluginSourceOutputData& OutputData)
{
	FScopeLock ScopeLock(&Objects[InputData.SourceId].ObjectCritSect);

	FSpatialSoundSourceObjectData& ObjectData = Objects[InputData.SourceId];
	check(InputData.AudioBuffer != nullptr);

	if (!ObjectData.bActive && !ObjectData.bBuffering)
	{
		ObjectData.AudioBuffer.SetCapacity(4096 * 50);
		ObjectData.bBuffering = true;
	}

	int32 NumSamplesToPush = InputData.AudioBuffer->Num();
	int32 SamplesWritten = ObjectData.AudioBuffer.Push(InputData.AudioBuffer->GetData(), NumSamplesToPush);

	if (SamplesWritten != NumSamplesToPush)
	{
		UE_LOGF(LogMicrosoftSpatialSound, Warning, "Source circular buffers should be bigger!");
	}

	FVector NewPosition = UnrealToMicrosoftSpatialSoundCoordinates(InputData.SpatializationParams->EmitterPosition, InputData.SpatializationParams->Distance);

	if (ObjectData.bBuffering && ObjectData.AudioBuffer.Num() > MinFramesRequiredPerObjectUpdate * 5)
	{
		ObjectData.bBuffering = false;
	}

	bool bIsFirstPosition = false;
	if (!ObjectData.bBuffering && !ObjectData.bActive)
	{
		bIsFirstPosition = true;
		ObjectData.bActive = true;
		ObjectData.ObjectHandle = nullptr;

		ObjectData.ObjectHandle = SAC->ActivatDynamicSpatialAudioObject();
	}

	if (bIsFirstPosition || !FMath::IsNearlyEqual(ObjectData.TargetPosition.X, NewPosition.X) || !FMath::IsNearlyEqual(ObjectData.TargetPosition.Y, NewPosition.Y) || !FMath::IsNearlyEqual(ObjectData.TargetPosition.Z, NewPosition.Z))
	{
		if (bIsFirstPosition)
		{
			ObjectData.StartingPosition = ObjectData.TargetPosition;
		}
		else
		{
			ObjectData.StartingPosition = ObjectData.CurrentPosition;
		}
		ObjectData.TargetPosition = NewPosition;
		ObjectData.CurrentFrameLerpPosition = 0;
		ObjectData.NumberOfLerpFrames = 4*NumSamplesToPush;
	}
}

uint32 FMicrosoftSpatialSound::Run()
{
	HRESULT Result = S_OK;

	// Wait until the SAC is active
	while (!SAC->IsActive() && bIsRendering)
	{
		if (!bWarnedMicrosoftSpatialSoundDynamicObjectCountIsZero && SAC->GetMaxDynamicObjects() == 0)
		{
			UE_LOGF(LogMicrosoftSpatialSound, Warning, "Microsoft Spatial Sound has zero MaxDynamicObjects.  No sounds can play!  You need to enable Spatial Sound (Windows Sonic for Headphones) in your PC audio settings then restart Unreal.");
			bWarnedMicrosoftSpatialSoundDynamicObjectCountIsZero = true;
		}

		FPlatformProcess::Sleep(0.01f);
	}

	// The render loop
	while (bIsRendering)
	{
		if (SAC->WaitTillBufferCompletionEvent())
		{
			//UE_LOGF(LogMicrosoftSpatialSound, Warning, "Microsoft Spatial Sound buffer completion event timed out.");
		}

		// We need to lock while updating the spatial renderer		
		uint32 FrameCount = 0;
		uint32 AvailableObjects = 0;

		SAC->BeginUpdating(&AvailableObjects, &FrameCount);

		for (FSpatialSoundSourceObjectData& Object : Objects)
		{
			FScopeLock ScopeLock(&Object.ObjectCritSect);

			if (Object.bActive)
			{
				if (Object.AudioBuffer.Num() > FrameCount)
				{
					// Get the buffer for the audio object
					float* OutBuffer = nullptr;
					uint32 BufferLength = 0;
					Result = Object.ObjectHandle->GetBuffer((BYTE**)&OutBuffer, &BufferLength);
					MS_SPATIAL_AUDIO_LOG_ON_FAIL(Result);

					// Fill that buffer from the circular buffer queue
					uint32 NumSamples = BufferLength / sizeof(float);
					Object.AudioBuffer.Pop(OutBuffer, NumSamples);

					Result = Object.ObjectHandle->SetVolume(1.0f);
					MS_SPATIAL_AUDIO_LOG_ON_FAIL(Result);

					float LerpFraction = FMath::Clamp((float)Object.CurrentFrameLerpPosition / Object.NumberOfLerpFrames, 0.0f, 1.0f);

					Object.CurrentPosition = FMath::Lerp(Object.StartingPosition, Object.TargetPosition, LerpFraction);
					// Update the object's meta-data 
					Result = Object.ObjectHandle->SetPosition(Object.CurrentPosition.X, Object.CurrentPosition.Y, Object.CurrentPosition.Z);
					MS_SPATIAL_AUDIO_LOG_ON_FAIL(Result);

					Object.CurrentFrameLerpPosition += NumSamples;
				}
				else
				{
					UE_LOGF(LogTemp, Log, "Foo");
					float* OutBuffer = nullptr;
					uint32 BufferLength = 0;
					Result = Object.ObjectHandle->GetBuffer((BYTE**)&OutBuffer, &BufferLength);
					MS_SPATIAL_AUDIO_LOG_ON_FAIL(Result);
				}
			}
		}

		SAC->EndUpdating();
	}

	FreeResources();
	return 0;
}

#if PLATFORM_CPU_ARM_FAMILY && !PLATFORM_WINDOWS_ARM64EC
	#define TARGET_ARCH TEXT("arm64")
#else
	#define TARGET_ARCH TEXT("x64")
#endif

void FMicrosoftSpatialSoundModule::StartupModule()
{
	const FString LibraryName = "SpatialAudioClientInterop";

	const FString DllName = FString::Printf(TEXT("%s.dll"), *LibraryName);

#if UE_BUILD_DEBUG && !defined(NDEBUG)	// Use !defined(NDEBUG) to check to see if we actually are linking with Debug third party libraries (bDebugBuildsActuallyUseDebugCRT)
	const FString ConfigName = "Debug";
#else
	const FString ConfigName = "Release";
#endif

	const FString ThirdPartyDir = FPaths::EngineDir() / "Binaries/ThirdParty" / LibraryName / FPlatformProperties::IniPlatformName() / ConfigName / TARGET_ARCH;

	LibraryHandle = FPlatformProcess::GetDllHandle(*(ThirdPartyDir / DllName));
	if (LibraryHandle == nullptr)
	{
		UE_LOGF(LogMicrosoftSpatialSound, Warning, "Failed to load the SpatialAudioClientInterop DLL");
	}

	IModularFeatures::Get().RegisterModularFeature(FMicrosoftSpatialSoundPluginFactory::GetModularFeatureName(), &PluginFactory);
}

void FMicrosoftSpatialSoundModule::ShutdownModule()
{
	if (LibraryHandle != nullptr)
	{
		FPlatformProcess::FreeDllHandle(LibraryHandle);
	}
}

IMPLEMENT_MODULE(FMicrosoftSpatialSoundModule, MicrosoftSpatialSound)