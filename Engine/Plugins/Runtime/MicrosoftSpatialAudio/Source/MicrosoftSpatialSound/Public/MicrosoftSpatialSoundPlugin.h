// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Modules/ModuleManager.h"
#include "IAudioExtensionPlugin.h"
#include "DSP/Dsp.h"
#include "HAL/Runnable.h"
#include "HAL/RunnableThread.h"
#include "Templates/Function.h"
#include "Containers/Queue.h"
#include "SpatialAudioClientInterop.h"

// Struct to hold dynamic object data for Microsoft Spatial Sound API
struct FSpatialSoundSourceObjectData
{
	FSpatialSoundSourceObjectData()
		: StartingPosition(FVector::ZeroVector)
		, CurrentPosition(FVector::ZeroVector)
		, TargetPosition(FVector::ZeroVector)
		, CurrentFrameLerpPosition(0)
		, NumberOfLerpFrames(0)
		, AudioBuffer(4096*50)
		, ObjectHandle(nullptr)
		, bActive(false)
		, bBuffering(false)
	{
	}

	// The position of the spatial sound source
	FVector StartingPosition;
	FVector CurrentPosition;
	FVector TargetPosition;
	int32 CurrentFrameLerpPosition;
	int32 NumberOfLerpFrames;

	// A circular buffer to hold audio data from the source.
	Audio::TCircularAudioBuffer<float> AudioBuffer;

	// The handle to the source
	ISpatialAudioObject* ObjectHandle;

	FCriticalSection ObjectCritSect;

	// Whether or not this source data is active (playing sound)
	bool bActive;
	bool bBuffering;
};

// Implementation of IAudioSpatialization for Microsoft Spatial Sound API
class FMicrosoftSpatialSound : 
	public IAudioSpatialization,
	protected FRunnable						
{
public:
	FMicrosoftSpatialSound();
	~FMicrosoftSpatialSound();

	/** Begin IAudioSpatialization interface */
	virtual void Initialize(const FAudioPluginInitializationParams InitializationParams) override;
	virtual void Shutdown() override;
	virtual bool IsSpatializationEffectInitialized() const override { return bIsInitialized; };
	virtual void OnInitSource(const uint32 SourceId, const FName& AudioComponentUserId, USpatializationPluginSourceSettingsBase* InSettings) override;
	virtual void OnReleaseSource(const uint32 SourceId) override;
	virtual void ProcessAudio(const FAudioPluginSourceInputData& InputData, FAudioPluginSourceOutputData& OutputData);
	/** End of IAudioSpatialization interface */

	/** Begin FRunnable */
	virtual uint32 Run() override;
	/** End FRunnable */

private:

	void FreeResources();

	FAudioPluginInitializationParams InitializationParams;

	// Command queue
	TQueue<TFunction<void()>> CommandQueue;

	// The minimum amount of frames which need to be queued before processing audio objects
	uint32 MinFramesRequiredPerObjectUpdate;

	// Array of spatial sound source objects
	TArray<FSpatialSoundSourceObjectData> Objects;

	// The max dynamic objects
	uint32 MaxDynamicObjects;

	// The spatial audio render thread
	FRunnableThread* SpatialAudioRenderThread;

	// The rendering critical section
	FCriticalSection SpatialSoundCritSect;

	// If the spatial audio render thread is running
	FThreadSafeBool bIsRendering;

	SpatialAudioClient* SAC;

	bool bIsInitialized;

};

class FMicrosoftSpatialSoundPluginFactory : public IAudioSpatializationFactory
{
public:

	virtual FString GetDisplayName() override
	{
		static FString DisplayName = FString(TEXT("Microsoft Spatial Sound"));
		return DisplayName;
	}

	virtual bool SupportsPlatform(const FString& PlatformName) override
	{
		return (PlatformName == TEXT("Windows")) || (PlatformName == TEXT("XboxOne"));
	}

	// Microsoft spatial sound dynamic objects render objects externally from the audio renderer
	virtual bool IsExternalSend() override { return true; };

	virtual TAudioSpatializationPtr CreateNewSpatializationPlugin(FAudioDevice* OwningDevice) override
	{ 
		return TAudioSpatializationPtr(new FMicrosoftSpatialSound());
	};
};

class FMicrosoftSpatialSoundModule : public IModuleInterface
{
public:
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;

private:
	void* LibraryHandle = nullptr;
	FMicrosoftSpatialSoundPluginFactory PluginFactory;
};
